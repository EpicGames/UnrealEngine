// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class FbxAutomationTestBuilder : ModuleRules
{
    public FbxAutomationTestBuilder(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange
        (
            new string[] {
				"Core",
                "InputCore",
                "CoreUObject",
				"Slate",
                
                "Engine",
                "UnrealEd",
                "PropertyEditor",
				"LevelEditor"
            }
        );
        
        PrivateDependencyModuleNames.AddRange(
             new string[] {
					"Engine",
                    "UnrealEd",
                    "WorkspaceMenuStructure"
                }
         );

        if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
                    "UnrealEd",
    				"SlateCore",
    				"Slate",
                }
            );
        }
	}
}
