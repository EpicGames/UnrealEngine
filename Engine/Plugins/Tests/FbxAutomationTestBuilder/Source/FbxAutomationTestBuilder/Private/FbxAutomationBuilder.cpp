// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxAutomationBuilder.h"
#include "DetailsViewArgs.h"
#include "Misc/MessageDialog.h"
#include "Framework/Views/TableViewMetadata.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Factories/FbxImportUI.h"
#include "Editor.h"

#include "PropertyEditorModule.h"
#include "IDetailsView.h"

#include "Tests/FbxAutomationCommon.h"

#define PIXEL_INSPECTOR_REQUEST_TIMEOUT 10
#define MINIMUM_TICK_BETWEEN_CREATE_REQUEST 10
#define LOCTEXT_NAMESPACE "FbxAutomationBuilder"

namespace FbxAutomationBuilder
{
	SFbxAutomationBuilder::SFbxAutomationBuilder()
	{
		CurrentPlan = nullptr;
		CurrentPlanModified = false;
		TestPlanDetailsView = nullptr;

		OnEditorCloseHandle = GEditor->OnEditorClose().AddRaw(this, &SFbxAutomationBuilder::ReleaseRessource);
	}

	SFbxAutomationBuilder::~SFbxAutomationBuilder()
	{
		ReleaseRessource();
	}

	void SFbxAutomationBuilder::ReleaseRessource()
	{
		FlushAllPlan();

		if (OnEditorCloseHandle.IsValid())
		{
			GEditor->OnEditorClose().Remove(OnEditorCloseHandle);
			OnEditorCloseHandle = FDelegateHandle();
		}

		if (TestPlanDetailsView.IsValid())
		{
			TestPlanDetailsView->SetObject(nullptr);
			TestPlanDetailsView = nullptr;
		}
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void SFbxAutomationBuilder::Construct(const FArguments& InArgs)
	{
		JsonFileIsReadOnly = false;
		TSharedPtr<SBox> InspectorBox;

		//Create the FbxAutomationBuilder UI
		TSharedPtr<SVerticalBox> VerticalBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.FillWidth(0.25)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FbxSelection", "Select a fbx file"))
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.FillWidth(0.75)
			.VAlign(VAlign_Center)
			[
				SAssignNew(FbxFilesCombo, SComboButton)
				.OnGetMenuContent(this, &SFbxAutomationBuilder::OnGetFbxMenuContent)
				.ContentPadding(1)
				.ToolTipText(LOCTEXT("FbxComboBox", "Select a fbx file."))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &SFbxAutomationBuilder::OnGetFbxListButtonText)
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.FillWidth(0.25)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PlanSelection", "Select a test plan"))
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.FillWidth(0.75)
			.VAlign(VAlign_Center)
			[
				SAssignNew(PlanCombo, SComboButton)
				.OnGetMenuContent(this, &SFbxAutomationBuilder::OnGetPlanMenuContent)
				.ContentPadding(1)
				.ToolTipText(LOCTEXT("PlanComboBox", "Select a test plan."))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &SFbxAutomationBuilder::OnGetPlanListButtonText)
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("SaveJsonFile", "Save JSON"))
				.ToolTipText(LOCTEXT("SaveJsonFileTooltip", "Save all Plan for the current Fbx"))
				.OnClicked(this, &SFbxAutomationBuilder::OnSaveJSON)
				.IsEnabled(this, &SFbxAutomationBuilder::CanSavePlans)
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("DeleteCurrentPlan", "Delete CurrentPlan"))
				.ToolTipText(LOCTEXT("DeleteCurrentPlanTooltip", "Delete the current selected plan"))
				.OnClicked(this, &SFbxAutomationBuilder::OnDeleteCurrentPlan)
				.IsEnabled(this, &SFbxAutomationBuilder::CanSavePlans)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(InspectorBox, SBox)
		];

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bSearchInitialKeyFocus = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		TestPlanDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		InspectorBox->SetContent(TestPlanDetailsView->AsShared());
		TestPlanDetailsView->OnFinishedChangingProperties().AddSP(this, &SFbxAutomationBuilder::OnFinishedChangingPlanProperties);

		//Add the vertical box to the child slot
		ChildSlot
		[
			VerticalBox.ToSharedRef()
		];
	}

	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	FReply SFbxAutomationBuilder::OnSaveJSON()
	{
		FString Ext = FPaths::GetExtension(CurrentFbx, true);
		FString FileOptionAndResult = CurrentFbx;
		if (FileOptionAndResult.RemoveFromEnd(*Ext))
		{
			FileOptionAndResult += TEXT(".json");
			FbxAutomationTestsAPI::WriteFbxOptions(FileOptionAndResult, AllPlans);
			CurrentPlanModified = false;
		}
		return FReply::Handled();
	}

	FReply SFbxAutomationBuilder::OnDeleteCurrentPlan()
	{
		if (CurrentPlan != nullptr)
		{
			if (AllPlans.Contains(CurrentPlan))
			{
				AllPlans.Remove(CurrentPlan);
			}
			CurrentPlan->RemoveFromRoot();
			CurrentPlan = nullptr;
			CurrentPlanModified = false;
			TestPlanDetailsView->SetObject(nullptr);
		}
		return FReply::Handled();
	}

	void SFbxAutomationBuilder::OnFinishedChangingPlanProperties(const FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (JsonFileIsReadOnly)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("TellTestPlanIsReadOnly", "The test plan you try to modify is read only, you will not be able to save your changes."));
		}
		else
		{
			CurrentPlanModified = true;
		}
	}

	void SFbxAutomationBuilder::FlushAllPlan()
	{
		for (UFbxTestPlan *TestPlan : AllPlans)
		{
			TestPlan->RemoveFromRoot();
		}
		AllPlans.Empty();
		JsonFileIsReadOnly = false;
		CurrentPlan = nullptr;
		CurrentPlanModified = false;
		if (TestPlanDetailsView.IsValid())
		{
			TestPlanDetailsView->SetObject(nullptr);
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// FBX Combo box

	FText SFbxAutomationBuilder::OnGetFbxListButtonText() const
	{
		if (!CurrentFbx.IsEmpty())
		{
			return FText::FromString(CurrentFbx);
		}
		return LOCTEXT("OnGetFbxListButtonText", "No fbx Selected");
	}

	TSharedRef<SWidget> SFbxAutomationBuilder::OnGetFbxMenuContent()
	{
		if (AllPlans.Num() > 0 && CurrentPlan != nullptr && CurrentPlanModified)
		{
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("AskForSave", "Do you want to save your data before reading other test.")) == EAppReturnType::Yes)
			{
				OnSaveJSON();
			}
		}
		// List	all existing plan
		ComboBoxExistingFbx.Empty();
		FlushAllPlan();

		FString ImportTestDirectory;
		check(GConfig);
		GConfig->GetString(TEXT("AutomationTesting.FbxImport"), TEXT("FbxImportEditorTestPath"), ImportTestDirectory, GEngineIni);


		// Find all files in the GenericImport directory
		TArray<FString> FilesInDirectory;
		IFileManager::Get().FindFilesRecursive(FilesInDirectory, *ImportTestDirectory, TEXT("*.*"), true, false);

		// Scan all the found files, use only .fbx file
		for (TArray<FString>::TConstIterator FileIter(FilesInDirectory); FileIter; ++FileIter)
		{
			FString Filename(*FileIter);
			FString Ext = FPaths::GetExtension(Filename, true);
			if (Ext.Compare(TEXT(".fbx"), ESearchCase::IgnoreCase) == 0)
			{
				FString FileString = *FileIter;
				FString FileBaseName = FPaths::GetBaseFilename(Filename);
				if (FileBaseName.Len() > 6)
				{
					FString FileEndSuffixe = FileBaseName.RightChop(FileBaseName.Find(TEXT("_lod"), ESearchCase::IgnoreCase, ESearchDir::FromEnd));
					FString LodNumber = FileEndSuffixe.RightChop(4);
					FString LodBaseSuffixe = FileEndSuffixe.LeftChop(2);
					if (LodBaseSuffixe.Compare(TEXT("_lod"), ESearchCase::IgnoreCase) == 0)
					{
						if (LodNumber.Compare(TEXT("00")) != 0)
						{
							//Don't add lodmodel has test
							continue;
						}
					}
				}
				if (FileBaseName.Len() > 4 && FileBaseName.EndsWith(TEXT("_alt"), ESearchCase::IgnoreCase))
				{
					//Dont add alternate skinning file has test
					continue;
				}
				ComboBoxExistingFbx.Add(MakeShareable(new FString(FileString)));
			}
		}

		TSharedRef<SWidget> ContextMenuWidget = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SListView< TSharedRef<FString> >)
			.ListItemsSource(&ComboBoxExistingFbx)
			.OnGenerateRow(this, &SFbxAutomationBuilder::OnGenerateFbxRow)
			.OnSelectionChanged(this, &SFbxAutomationBuilder::OnFbxSelected)
		];
		return ContextMenuWidget;
	}

	TSharedRef<ITableRow> SFbxAutomationBuilder::OnGenerateFbxRow(TSharedRef<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		FString ItemString = (*InItem).IsEmpty() ? TEXT("Invalid Filename") : (*InItem);
		return
			SNew(SComboRow< TSharedRef<FString> >, OwnerTable)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(ItemString))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
			];
	}

	void SFbxAutomationBuilder::OnFbxSelected(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo)
	{
		CurrentFbx = (*InItem).IsEmpty() ? LOCTEXT("OnGenerateFbxRow_InvalidItem", "Invalid Filename").ToString() : *InItem;
		FbxFilesCombo->SetIsOpen(false);
	}

	//////////////////////////////////////////////////////////////////////////
	// PLAN Combo box

	FText SFbxAutomationBuilder::OnGetPlanListButtonText() const
	{
		if (CurrentPlan != nullptr)
		{
			FString PlanNameDisplay = CurrentPlan->TestPlanName + (CurrentPlanModified ? TEXT("*") : TEXT(""));
			if (JsonFileIsReadOnly)
			{
				PlanNameDisplay = PlanNameDisplay + TEXT(" [ReadOnly]");
			}
			return FText::FromString(PlanNameDisplay);
		}
		return LOCTEXT("OnGetPlanListButtonText", "No Plan Selected");
	}

	TSharedRef<SWidget> SFbxAutomationBuilder::OnGetPlanMenuContent()
	{
		if (AllPlans.Num() > 0 && CurrentPlan != nullptr && CurrentPlanModified)
		{
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("AskForSave", "Do you want to save your data before reading other test.")) == EAppReturnType::Yes)
			{
				OnSaveJSON();
			}
		}
		// List	all existing plan
		ComboBoxExistingPlan.Empty();
		FlushAllPlan();

		FString Ext = FPaths::GetExtension(CurrentFbx, true);
		FString FileOptionAndResult = CurrentFbx;
		if (FileOptionAndResult.RemoveFromEnd(*Ext))
		{
			FileOptionAndResult += TEXT(".json");

			TArray<UFbxTestPlan*> TestPlanArray;
			if (IFileManager::Get().FileExists(*FileOptionAndResult))
			{
				JsonFileIsReadOnly = IFileManager::Get().IsReadOnly(*FileOptionAndResult);
				if (!JsonFileIsReadOnly)
				{
					ComboBoxExistingPlan.Add(MakeShareable(new FString(TEXT("Create new plan"))));
				}

				//Read the fbx options from the .json file and fill the ImportUI
				FbxAutomationTestsAPI::ReadFbxOptions(FileOptionAndResult, TestPlanArray);

				for (UFbxTestPlan* TestPlan : TestPlanArray)
				{
					AllPlans.Add(TestPlan);
					ComboBoxExistingPlan.Add(MakeShareable(new FString(TestPlan->TestPlanName)));
				}
			}
			else
			{
				//Make sure the Create new plan is there
				ComboBoxExistingPlan.Add(MakeShareable(new FString(TEXT("Create new plan"))));
			}
		}
		else
		{
			//Make sure the Create new plan is there
			ComboBoxExistingPlan.Add(MakeShareable(new FString(TEXT("Create new plan"))));
		}

		TSharedRef<SWidget> ContextMenuWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SListView< TSharedRef<FString> >)
				.ListItemsSource(&ComboBoxExistingPlan)
			.OnGenerateRow(this, &SFbxAutomationBuilder::OnGeneratePlanRow)
			.OnSelectionChanged(this, &SFbxAutomationBuilder::OnPlanSelected)
			];
		return ContextMenuWidget;
	}

	TSharedRef<ITableRow> SFbxAutomationBuilder::OnGeneratePlanRow(TSharedRef<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		FString ItemString = (*InItem).IsEmpty() ? TEXT("Invalid Plan name") : (*InItem);
		return
			SNew(SComboRow< TSharedRef<FString> >, OwnerTable)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(ItemString))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
			];
	}

	void SFbxAutomationBuilder::OnPlanSelected(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo)
	{
		CurrentPlan = nullptr;
		if (!(*InItem).IsEmpty())
		{
			if (!JsonFileIsReadOnly && FString(TEXT("Create new plan")).Compare((*InItem)) == 0)
			{
				CurrentPlan = NewObject<UFbxTestPlan>();
				CurrentPlan->AddToRoot();
				CurrentPlan->ImportUI = NewObject<UFbxImportUI>();
				CurrentPlan->ImportUI->AddToRoot();
				CurrentPlan->TestPlanName = TEXT("Enter a plan name");
				CurrentPlanModified = true;
				AllPlans.Add(CurrentPlan);
				if (TestPlanDetailsView.IsValid())
				{
					TestPlanDetailsView->SetObject(CurrentPlan, true);
				}
			}
			else
			{
				CurrentPlan = nullptr;
				for (UFbxTestPlan *TestPlan : AllPlans)
				{
					if ((*InItem).Compare(TestPlan->TestPlanName) == 0)
					{
						CurrentPlan = TestPlan;
						break;
					}
				}
				TestPlanDetailsView->SetObject(CurrentPlan, true);
				CurrentPlanModified = false;
			}
		}
		PlanCombo->SetIsOpen(false);
	}

	//////////////////////////////////////////////////////////////////////////

	FText SFbxAutomationBuilder::GetPlanTextName() const
	{
		return LOCTEXT("GetActivePlanTextName", "Enter plan name here");
	}

	void SFbxAutomationBuilder::OnPlanNameChanged(const FText& NewName, ETextCommit::Type CommitInfo)
	{
	}

	void SFbxAutomationBuilder::OnPlanReimportStateChanged(ECheckBoxState InState)
	{
	}

	ECheckBoxState SFbxAutomationBuilder::IsPlanReimportChecked() const
	{
		return ECheckBoxState::Unchecked;
	}
};

#undef LOCTEXT_NAMESPACE
