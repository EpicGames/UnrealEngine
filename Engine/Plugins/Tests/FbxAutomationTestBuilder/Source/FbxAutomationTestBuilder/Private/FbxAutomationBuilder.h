// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
enum class ECheckBoxState : uint8;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

class IDetailsView;
class SComboButton;
class UFbxTestPlan;
struct FPropertyChangedEvent;

//#include "FbxAutomationTests.h"

class IDetailsView;
class UFbxTestPlan;

namespace FbxAutomationBuilder
{
#define WAIT_FRAMENUMBER_BEFOREREADING 5
	/**
	* Implements the FbxAutomationBuilder window.
	*/
	class SFbxAutomationBuilder : public SCompoundWidget, public FNotifyHook
	{

	public:
		/** Default constructor. */
		SFbxAutomationBuilder();

		/** Virtual destructor. */
		virtual ~SFbxAutomationBuilder();

		SLATE_BEGIN_ARGS(SFbxAutomationBuilder){}
		SLATE_END_ARGS()

		/**
		* Constructs this widget.
		*/
		void Construct(const FArguments& InArgs);

		

	private:

		FDelegateHandle OnEditorCloseHandle;
		void ReleaseRessource();

		FText GetSaveButtonText() const;

		bool CanSavePlans() const { return !JsonFileIsReadOnly; }

		//Save the JSON file
		FReply OnSaveJSON();
		// Delete the current edited plan
		FReply OnDeleteCurrentPlan();

		void FlushAllPlan();

		void OnFinishedChangingPlanProperties(const FPropertyChangedEvent& PropertyChangedEvent);

		//////////////////////////////////////////////////////////////////////////
		// Fbx files
		TArray<TSharedRef<FString>> ComboBoxExistingFbx;
		FString CurrentFbx;

		TSharedPtr<SComboButton> FbxFilesCombo;

		TSharedRef<ITableRow> OnGenerateFbxRow(TSharedRef<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

		void OnFbxSelected(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo);

		FText OnGetFbxListButtonText() const;

		TSharedRef<SWidget> OnGetFbxMenuContent();

		//////////////////////////////////////////////////////////////////////////
		// Test plan 
		TArray<TSharedRef<FString>> ComboBoxExistingPlan;
		TArray<UFbxTestPlan*> AllPlans;
		bool JsonFileIsReadOnly;
		UFbxTestPlan* CurrentPlan;
		bool CurrentPlanModified;

		TSharedPtr<class SComboButton> PlanCombo;

		TSharedRef<ITableRow> OnGeneratePlanRow(TSharedRef<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

		void OnPlanSelected(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo);

		FText OnGetPlanListButtonText() const;

		TSharedRef<SWidget> OnGetPlanMenuContent();

		// The property editor DetailsView
		TSharedPtr<IDetailsView> TestPlanDetailsView;


		FText GetPlanTextName() const;

		void OnPlanNameChanged(const FText& NewName, ETextCommit::Type CommitInfo);

		void OnPlanReimportStateChanged(ECheckBoxState InState);

		ECheckBoxState IsPlanReimportChecked() const;

		void ReadExistingFbxTests();


	};
}
