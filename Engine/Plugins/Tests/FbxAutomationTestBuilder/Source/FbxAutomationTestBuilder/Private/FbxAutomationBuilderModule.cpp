// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxAutomationBuilderModule.h"
#include "FbxAutomationBuilder.h"
#include "FbxAutomationBuilderStyle.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "Widgets/Docking/SDockTab.h"

void FFbxAutomationBuilderModule::StartupModule()
{
	HFbxAutomationBuilderWindow = nullptr;
	bHasRegisteredTabSpawners = false;

	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	RegisterTabSpawner(MenuStructure.GetAutomationToolsCategory());
}

void FFbxAutomationBuilderModule::ShutdownModule()
{
	FFbxAutomationBuilderStyle::Shutdown();
}

TSharedRef<SWidget> FFbxAutomationBuilderModule::CreateFbxAutomationBuilderWidget()
{
	SAssignNew(HFbxAutomationBuilderWindow, FbxAutomationBuilder::SFbxAutomationBuilder);
	return HFbxAutomationBuilderWindow->AsShared();
}

void FFbxAutomationBuilderModule::RegisterTabSpawner(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup)
{
	if (bHasRegisteredTabSpawners)
	{
		UnregisterTabSpawner();
	}

	bHasRegisteredTabSpawners = true;
	
	FFbxAutomationBuilderStyle::Initialize();

	//Register the UFbxAutomationBuilderView detail customization
	{
		//FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		//PropertyModule.RegisterCustomClassLayout(UFbxAutomationBuilderView::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FFbxAutomationBuilderDetailsCustomization::MakeInstance));
		//PropertyModule.NotifyCustomizationModuleChanged();
	}

	{
		FTabSpawnerEntry& SpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("LevelEditorFbxAutomationBuilder", FOnSpawnTab::CreateRaw(this, &FFbxAutomationBuilderModule::MakeFbxAutomationBuilderTab))
			.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorFbxAutomationBuilder", "FBX Test Builder"))
			.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorFbxAutomationBuilderTooltipText", "Open the fbx automation test builder tool."))
			.SetIcon(FSlateIcon(FFbxAutomationBuilderStyle::Get()->GetStyleSetName(), "FbxAutomationBuilder.TabIcon"));

		if (WorkspaceGroup.IsValid())
		{
			SpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
		}
	}
}

void FFbxAutomationBuilderModule::UnregisterTabSpawner()
{
	bHasRegisteredTabSpawners = false;

	//Unregister the custom detail layout
	//if (UObjectInitialized())
	//{
	//	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	//	PropertyModule.UnregisterCustomClassLayout(UFbxAutomationBuilderView::StaticClass()->GetFName());
	//}

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("LevelEditorFbxAutomationBuilder");
}

TSharedRef<SDockTab> FFbxAutomationBuilderModule::MakeFbxAutomationBuilderTab(const FSpawnTabArgs&)
{
	TSharedRef<SDockTab> FbxAutomationBuilderTab = SNew(SDockTab)
	.TabRole(ETabRole::NomadTab);
	FbxAutomationBuilderTab->SetContent(CreateFbxAutomationBuilderWidget());
	return FbxAutomationBuilderTab;
}

IMPLEMENT_MODULE(FFbxAutomationBuilderModule, FbxAutomationTestBuilder);
