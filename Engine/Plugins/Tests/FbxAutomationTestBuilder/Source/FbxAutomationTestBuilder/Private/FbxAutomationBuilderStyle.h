// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FFbxAutomationBuilderStyle
{
public:

	static void Initialize();
	static void Shutdown();

	static TSharedPtr< class ISlateStyle > Get();

	static FName GetStyleSetName();

private:

	static TSharedPtr< class FSlateStyleSet > StyleSet;
};
