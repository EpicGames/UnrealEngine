// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/SWidget.h"

class FSpawnTabArgs;
class FWorkspaceItem;

namespace FbxAutomationBuilder { class SFbxAutomationBuilder; };

/**
* The module holding all of the UI related pieces for pixel inspector
*/
class FFbxAutomationBuilderModule : public IModuleInterface
{
public:
	/**
	* Called right after the module DLL has been loaded and the module object has been created
	*/
	virtual void StartupModule();

	/**
	* Called before the module is unloaded, right before the module object is destroyed.
	*/
	virtual void ShutdownModule();

	/** Creates the HLOD Outliner widget */
	virtual TSharedRef<SWidget> CreateFbxAutomationBuilderWidget();

	virtual void RegisterTabSpawner(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup);

	virtual void UnregisterTabSpawner();

private:
	TSharedRef<SDockTab> MakeFbxAutomationBuilderTab(const FSpawnTabArgs&);

	bool bHasRegisteredTabSpawners;
	TSharedPtr<FbxAutomationBuilder::SFbxAutomationBuilder> HFbxAutomationBuilderWindow;
};
